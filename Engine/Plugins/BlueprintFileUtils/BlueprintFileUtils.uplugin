{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Blueprint File Utilities",
	"Description": "A Blueprint library that enables low-level file operations such as Move, Copy, Delete, and Find.",
	"Category": "Blueprints",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "BlueprintFileUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}