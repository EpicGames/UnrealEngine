// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintFileUtilsBPLibrary.h"
#include "HAL/FileManager.h"

UBlueprintFileUtilsBPLibrary::UBlueprintFileUtilsBPLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

bool UBlueprintFileUtilsBPLibrary::FindFiles(const FString& Directory, TArray<FString>& FoundFiles, const FString& FileExtension)
{
	IFileManager::Get().FindFiles(FoundFiles, *Directory, *FileExtension);

	return FoundFiles.Num() > 0;
}

bool UBlueprintFileUtilsBPLibrary::FindRecursive(const FString& StartDirectory, TArray<FString>& FoundFilenames, const FString& FileExtension /*= TEXT("")*/, bool bFindFiles /*= true*/, bool bFindDirectories /*= false*/)
{
	const bool bClearFilenameArray = true;
	FString Wildcard = FileExtension;
	if (Wildcard.IsEmpty())
	{
		Wildcard = TEXT("*.*");
	}
	IFileManager::Get().FindFilesRecursive(FoundFilenames, *StartDirectory, *Wildcard, bFindFiles, bFindDirectories, bClearFilenameArray);
	return FoundFilenames.Num() > 0;
}

bool UBlueprintFileUtilsBPLibrary::FileExists(const FString& Filename)
{
	return IFileManager::Get().FileExists(*Filename);
}

bool UBlueprintFileUtilsBPLibrary::DirectoryExists(const FString& Directory)
{
	return IFileManager::Get().DirectoryExists(*Directory);
}

bool UBlueprintFileUtilsBPLibrary::MakeDirectory(const FString& Path, bool bCreateTree /*= false*/)
{
	return IFileManager::Get().MakeDirectory(*Path, bCreateTree);
}

bool UBlueprintFileUtilsBPLibrary::DeleteDirectory(const FString& Directory, bool bMustExist /*= false*/, bool bDeleteRecursively /*= false*/)
{
	return IFileManager::Get().DeleteDirectory(*Directory, bMustExist, bDeleteRecursively);
}

bool UBlueprintFileUtilsBPLibrary::DeleteFile(const FString& Filename, bool bMustExist /*= false*/, bool bEvenIfReadOnly /*= false*/)
{
	return IFileManager::Get().Delete(*Filename, bMustExist, bEvenIfReadOnly);
}

bool UBlueprintFileUtilsBPLibrary::CopyFile(const FString& DestFilename, const FString& SrcFilename, bool bReplace /*= true*/, bool bEvenIfReadOnly /*= false*/)
{
	return IFileManager::Get().Copy(*DestFilename, *SrcFilename, bReplace, bEvenIfReadOnly) == COPY_OK;
}

bool UBlueprintFileUtilsBPLibrary::MoveFile(const FString& DestFilename, const FString& SrcFilename, bool bReplace /*= true*/, bool bEvenIfReadOnly /*= false*/)
{
	return IFileManager::Get().Move(*DestFilename, *SrcFilename, bReplace, bEvenIfReadOnly);
}

FString UBlueprintFileUtilsBPLibrary::GetUserDirectory()
{
	return FPlatformProcess::UserDir();
}
