// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintFileUtilsBPLibrary.generated.h"

UCLASS()
class UBlueprintFileUtilsBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 * Finds all the files within the given directory, with optional file extension filter.
	 *
	 * @param Directory		The absolute path to the directory to search. Ex: "C:\UnrealEditor\Pictures"
	 * @param FoundFiles	All the files found that matched the optional FileExtension filter, or all files if none was specified.
	 * @param FileExtension	If FileExtension is empty string "" then all files are found.
	 * 						Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 * @return				true if anything was found, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool FindFiles(const FString& Directory, TArray<FString>& FoundFiles, const FString& FileExtension = TEXT(""));

	/**
	 * Finds all the files and/or directories within the given directory and any sub-directories.  Files can be found with anoptional file extension filter.
	 *
	 * @param StartDirectory	The absolute path to the directory to start the search. Ex: "C:\UnrealEditor\Pictures"
	 * @param FoundPaths		All the paths (directories and/or files) found
	 * @param Wildcard			Wildcard that can be used to find files or directories with specific text in their name.  
								E.g *.png to find all files ending with the png extension, *images* to find anything with the word "images" in it
	 * 							Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 *							Does not apply to directories
	 * @param bFindFiles		Whether or not to find files
	 * @param bFindDirectories	Whether or not to find directories
	 * @return					true if anything was found, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool FindRecursive(const FString& StartDirectory, TArray<FString>& FoundPaths, const FString& Wildcard = TEXT(""), bool bFindFiles = true, bool bFindDirectories = false);

	/** 
	 * Checks if a file exists
	 * 
	 * @param Filename	The filename to check 
	 * @return			true if Filename exists, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool FileExists(const FString& Filename);

	/**
	 * Checks if a directory exists
	 *
	 * @param Directory		The directory path to check
	 * @return				true if Directory exists, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool DirectoryExists(const FString& Directory);

	/**
	 * Makes a new directory, and optionally sub-directories
	 *
	 * @param Path			The directory path to make
	 * @param bCreateTree	If true, the entire directory tree will be created if it doesnt exist.  Otherwise only the leaft most directory will be created if possible
	 * @return				true if the directory was created, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool MakeDirectory(const FString& Path, bool bCreateTree = false);

	/**
	 * Deletes a directory and all the files in it and optionally all sub-directories and files within it
	 *
	 * @param Directory				The Directory to delete
	 * @param bMustExist			If true, the directory must exist or the return value will be false
	 * @param bDeleteRecursively	If true, all sub-directories will be deleted as well.  
									If false and there are contents in the directory, the delete operation will fail.
	 * @return						true if the directory was succesfully deleted, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool DeleteDirectory(const FString& Directory, bool bMustExist = false, bool bDeleteRecursively = false);

	/** Deletes a file. */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool DeleteFile(const FString& Filename, bool bMustExist = false, bool bEvenIfReadOnly = false);

	/** Copies a file. */
	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool CopyFile(const FString& DestFilename, const FString& SrcFilename, bool bReplace = true, bool bEvenIfReadOnly = false);

	UFUNCTION(BlueprintCallable, Category = "FileUtils")
	static bool MoveFile(const FString& DestFilename, const FString& SrcFilename, bool bReplace = true, bool bEvenIfReadOnly = false);

	/** Get the users directory.  Platform specific (usually something like MyDocuments or the users home directory */
	UFUNCTION(BlueprintPure, Category = "FileUtils")
	static FString GetUserDirectory();

};
